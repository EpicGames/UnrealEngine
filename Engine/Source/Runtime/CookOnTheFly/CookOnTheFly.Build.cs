// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CookOnTheFly : ModuleRules
{
	public CookOnTheFly(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
				"CoreUObject",
				"Sockets",
                "Networking"
            }
        );
    }
}
