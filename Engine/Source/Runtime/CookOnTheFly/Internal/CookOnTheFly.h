// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Async/Future.h"
#include "Misc/Timespan.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Delegates/Delegate.h"

class FArrayReader;
class FInternetAddr;

COOKONTHEFLY_API DECLARE_LOG_CATEGORY_EXTERN(LogCookOnTheFly, Log, All);

namespace UE { namespace Cook
{

/**
 * Flags and message types to be used with the cook-on-the-fly server.
 *
 * The first 8 bits specifies the message type, i.e request, response or a on-way message.
 */
enum class ECookOnTheFlyMessage : uint32
{
	/* Represents no message. */
	None				= 0x00,
	
	/* A request message. */
	Request				= 0x02,
	/* A response message. */
	Response			= 0x04,
	TypeFlags			= 0x0F,

	/* Request to cook a package. */
	CookPackage			= 0x20,
	/* Get all currenlty cooked packages. */
	GetCookedPackages	= 0x30,
	/* Request to recompile shaders. */
	RecompileShaders	= 0x40,
	/* One way message indicating that one or more packages has been cooked. */
	PackagesCooked		= 0x50,
	/* One way message indicating that one or more files has been added. */
	FilesAdded			= 0x60,
	/* Request to recook packages. */
	RecookPackages		= 0x70,
	/* Legacy message for NetworkPlatformFile */
	NetworkPlatformFile	= 0x80,
	/* Request for a list of on-disk packages, which may or may not be cooked or cookable for a given platform */
	GetAllOnDiskPackages = 0x90
};
ENUM_CLASS_FLAGS(ECookOnTheFlyMessage);

/**
 * Returns a string from the specified message.
 */
inline const TCHAR* LexToString(ECookOnTheFlyMessage Message)
{
	EnumRemoveFlags(Message, ECookOnTheFlyMessage::TypeFlags);

	switch (Message)
	{
		case ECookOnTheFlyMessage::None:
			return TEXT("None");
		case ECookOnTheFlyMessage::CookPackage:
			return TEXT("CookPackage");
		case ECookOnTheFlyMessage::GetCookedPackages:
			return TEXT("GetCookedPackages");
		case ECookOnTheFlyMessage::RecompileShaders:
			return TEXT("RecompileShaders");
		case ECookOnTheFlyMessage::PackagesCooked:
			return TEXT("PackagesCooked");
		case ECookOnTheFlyMessage::FilesAdded:
			return TEXT("FilesAdded");
		case ECookOnTheFlyMessage::RecookPackages:
			return TEXT("RecookPackages");
		case ECookOnTheFlyMessage::NetworkPlatformFile:
			return TEXT("NetworkPlatformFile");
		default:
			return TEXT("Unknown");
	};
}

/**
 * Cook-on-the-fly message status.
 */
enum class ECookOnTheFlyMessageStatus : uint32
{
	/** No status. */
	None,
	/** The message is successful. */
	Ok,
	/** The message failed. */
	Error
};

/**
 * Returns a string from the specified message status.
 */
inline const TCHAR* LexToString(ECookOnTheFlyMessageStatus Status)
{
	switch (Status)
	{
		case ECookOnTheFlyMessageStatus::None:
			return TEXT("None");
		case ECookOnTheFlyMessageStatus::Ok:
			return TEXT("Ok");
		case ECookOnTheFlyMessageStatus::Error:
			return TEXT("Error");
		default:
			return TEXT("Unknown");
	}
}

/**
 * Cook-on-the-fly message header.
 */
struct FCookOnTheFlyMessageHeader
{
	/** Type of message */
	ECookOnTheFlyMessage MessageType = ECookOnTheFlyMessage::None;
	/** The message status. */
	ECookOnTheFlyMessageStatus MessageStatus = ECookOnTheFlyMessageStatus::Ok;
	/** Correlation id, used to match response with request. */
	uint32 CorrelationId = 0;
	/** When the message was sent. */
	int64 Timestamp = 0;

	COOKONTHEFLY_API FString ToString() const;

	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FCookOnTheFlyMessageHeader& Header);
};

/**
 * Cook-on-the-fly message.
 */
class FCookOnTheFlyMessage
{
public:
	/** Creates a new instance of a cook-on-the-fly message. */
	FCookOnTheFlyMessage() = default;

	/** Creates a new instance of a cook-on-the-fly message with the specified message type. */
	explicit FCookOnTheFlyMessage(ECookOnTheFlyMessage MessageType)
	{
		Header.MessageType = MessageType;
	}

	ECookOnTheFlyMessage GetMessageType() const
	{
		ECookOnTheFlyMessage MessageType = Header.MessageType;
		EnumRemoveFlags(MessageType, ECookOnTheFlyMessage::TypeFlags);
		return MessageType;
	}

	/** Returns the message header. */
	FCookOnTheFlyMessageHeader& GetHeader()
	{
		return Header;
	}

	/** Returns the message header. */
	const FCookOnTheFlyMessageHeader& GetHeader() const
	{
		return Header;
	}

	/** Set a new message header. */
	void SetHeader(const FCookOnTheFlyMessageHeader& InHeader)
	{
		Header = InHeader;
	}

	/** Sets the message status. */
	void SetStatus(ECookOnTheFlyMessageStatus InStatus)
	{
		Header.MessageStatus = InStatus;
	}

	/** Returns the message status. */
	ECookOnTheFlyMessageStatus GetStatus() const
	{
		return Header.MessageStatus;
	}

	/** Returns whether the message stauts is OK. */
	bool IsOk() const
	{
		return Header.MessageStatus == ECookOnTheFlyMessageStatus::Ok;
	}

	/** Set the message body. */
	COOKONTHEFLY_API void SetBody(TArray<uint8> InBody);

	/** Set body to serializable type. */
	template<typename BodyType>
	void SetBodyTo(BodyType InBody)
	{
		Body.Empty();
		FMemoryWriter Ar(Body);
		Ar << InBody;
	}
	
	/** Returns the message body. */
	TArray<uint8>& GetBody()
	{
		return Body;
	}

	/** Returns the message body. */
	const TArray<uint8>& GetBody() const
	{
		return Body;
	}

	/** Serialize the body as the specified type. */
	template<typename BodyType>
	BodyType GetBodyAs() const
	{
		BodyType Type;
		FMemoryReader Ar(Body);
		Ar << Type;

		return MoveTemp(Type);
	}

	/** Returns the total size of the message header and message body. */
	int64 TotalSize() const
	{
		return sizeof(FCookOnTheFlyMessageHeader) + Body.Num();
	}

	/** Creates an archive for ready the message body. */
	COOKONTHEFLY_API TUniquePtr<FArchive> ReadBody() const;

	/** Creates an archive for writing the message body. */
	COOKONTHEFLY_API TUniquePtr<FArchive> WriteBody();

	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FCookOnTheFlyMessage& Message);

protected:
	FCookOnTheFlyMessageHeader Header;
	TArray<uint8> Body;
};

/**
 * Cook-on-the-fly request.
 */
class FCookOnTheFlyRequest
	: public FCookOnTheFlyMessage
{
public:
	/** Creates a new instance of a cook-on-the-fly request. */
	FCookOnTheFlyRequest() = default;

	/** Creates a new instance of a cook-on-the-fly request with the specified request type. */
	explicit FCookOnTheFlyRequest(ECookOnTheFlyMessage MessageType)
		: FCookOnTheFlyMessage(MessageType | ECookOnTheFlyMessage::Request)
	{
	}
};

/**
 * Cook-on-the-fly response.
 */
class FCookOnTheFlyResponse
	: public FCookOnTheFlyMessage
{
public:
	/** Creates a new instance of a cook-on-the-fly response. */
	FCookOnTheFlyResponse() = default;

	/** Creates a new instance of a cook-on-the-fly response with the specified response type. */
	explicit FCookOnTheFlyResponse(ECookOnTheFlyMessage MessageType)
		: FCookOnTheFlyMessage(MessageType | ECookOnTheFlyMessage::Response)
	{
	}

	/** Creates a new instance of a cook-on-the-fly response for the specified request. */
	explicit FCookOnTheFlyResponse(const FCookOnTheFlyRequest& Request)
		: FCookOnTheFlyMessage(Request.GetMessageType())
	{
		Header.MessageType |= ECookOnTheFlyMessage::Response;
		Header.CorrelationId = Request.GetHeader().CorrelationId;
	}
};

/**
 * Cook-on-the-fly host address.
 */
struct FCookOnTheFlyHostOptions
{
	/** Host address. */
	TArray<FString> Hosts;
	/** How long to wait for the server to start. */
	FTimespan ServerStartupWaitTime;
};

class ICookOnTheFlyServerConnection
{
public:
	virtual ~ICookOnTheFlyServerConnection() = default;

	virtual const FString& GetHost() const = 0;

	virtual const FString& GetZenProjectName() const = 0;

	virtual const FString& GetPlatformName() const = 0;

	virtual const TArray<FString> GetZenHostNames() const = 0;

	virtual const uint16 GetZenHostPort() const = 0;

	/**
	 * Returns whether connected to the cook-on-the-fly server.
	 */
	virtual bool IsConnected() const = 0;

	virtual bool IsSingleThreaded() const = 0;

	/**
	 * Sends a request to the server.
	 *
	 * @param Request The request message to send.
	 */
	virtual TFuture<FCookOnTheFlyResponse> SendRequest(FCookOnTheFlyRequest& Request) = 0;

	/**
	 * Event triggered when a new message has been sent from the server.
	 */
	DECLARE_EVENT_OneParam(ICookOnTheFlyServerConnection, FMessageEvent, const FCookOnTheFlyMessage&);
	virtual FMessageEvent& OnMessage() = 0;
};

/**
 * Cook-on-the-fly module
 */
class ICookOnTheFlyModule
	: public IModuleInterface
{
public:
	virtual ~ICookOnTheFlyModule() = default;

	virtual TSharedPtr<ICookOnTheFlyServerConnection> GetDefaultServerConnection() = 0;

	/**
	 * Connect to the cook-on-the-fly server.
	 *
	 * @param HostOptions Cook-on-the-fly host options.
	 */
	virtual TUniquePtr<ICookOnTheFlyServerConnection> ConnectToServer(const FCookOnTheFlyHostOptions& HostOptions) = 0;
};

}} // namespace UE::Cook
