// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/PackageStore.h"

namespace UE { namespace ZenCookOnTheFly { namespace Messaging
{

struct FCompletedPackages
{
	TArray<FPackageStoreEntryResource> CookedPackages;
	TArray<FPackageId> FailedPackages;

	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FCompletedPackages& CompletedPackages);
};

struct FCookPackageRequest
{
	TArray<FPackageId> PackageIds;
	
	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FCookPackageRequest& Request);
};

using FCookPackageResponse = FCompletedPackages;

struct FRecookPackagesRequest
{
	TArray<FPackageId> PackageIds;

	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FRecookPackagesRequest& Request);
};

struct FRecookPackagesResponse
{
	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FRecookPackagesResponse& Response);
};

struct FGetAllOnDiskPackagesRequest
{
	TArray<FPackageId> PackageIds;

	COOKONTHEFLY_API friend FArchive& operator<<(FArchive& Ar, FGetAllOnDiskPackagesRequest& Request);
};

}}} // namesapce UE::ZenCookOnTheFly::Messaging
