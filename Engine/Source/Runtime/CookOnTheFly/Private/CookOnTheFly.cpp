// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFly.h"
#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/DateTime.h"
#include "HAL/FileManager.h"

namespace UE { namespace Cook
{

FString FCookOnTheFlyMessageHeader::ToString() const
{
	return FString::Printf(TEXT("Message='%s', Status='%s', CorrelationId='%u'"),
		LexToString(MessageType),
		LexToString(MessageStatus),
		CorrelationId);
}

FArchive& operator<<(FArchive& Ar, FCookOnTheFlyMessageHeader& Header)
{
	uint32 MessageType = static_cast<uint32>(Header.MessageType);
	uint32 MessageStatus = static_cast<uint32>(Header.MessageStatus);
	
	Ar << MessageType;
	Ar << MessageStatus;
	Ar << Header.CorrelationId;
	Ar << Header.Timestamp;

	if (Ar.IsLoading())
	{
		Header.MessageType = static_cast<ECookOnTheFlyMessage>(MessageType);
		Header.MessageStatus = static_cast<ECookOnTheFlyMessageStatus>(MessageStatus);
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCookOnTheFlyMessage& Message)
{
	Ar << Message.Header;
	Ar << Message.Body;

	return Ar;
}

void FCookOnTheFlyMessage::SetBody(TArray<uint8> InBody)
{
	Body = MoveTemp(InBody);
}

TUniquePtr<FArchive> FCookOnTheFlyMessage::ReadBody() const
{
	return MakeUnique<FMemoryReader>(Body);
}

TUniquePtr<FArchive> FCookOnTheFlyMessage::WriteBody()
{
	return MakeUnique<FMemoryWriter>(Body);
}

}} // namesapce UE::Cook
