// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFlyMessages.h"

namespace UE { namespace ZenCookOnTheFly { namespace Messaging
{

FArchive& operator<<(FArchive& Ar, FCompletedPackages& CompletedPackages)
{
	Ar << CompletedPackages.CookedPackages;
	Ar << CompletedPackages.FailedPackages;
	
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCookPackageRequest& Request)
{
	Ar << Request.PackageIds;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FRecookPackagesRequest& Request)
{
	Ar << Request.PackageIds;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FRecookPackagesResponse& Response)
{
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FGetAllOnDiskPackagesRequest& Request)
{ 
	Ar << Request.PackageIds;
	return Ar;
}

}}} // namesapce UE::ZenCookOnTheFly::Messaging
