// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFlyServerConnection.h"
#include "TCPTransport.h"
#include "PlatformTransport.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformHostCommunication.h"
#include "HAL/PlatformTime.h"
#include "Misc/CommandLine.h"

DEFINE_LOG_CATEGORY(LogCookOnTheFly);

class FCookOnTheFlyModule final
	: public UE::Cook::ICookOnTheFlyModule
{
public:

	//
	// IModuleInterface interface
	//

	virtual void StartupModule( ) override { }

	virtual void ShutdownModule( ) override { }
	
	virtual bool SupportsDynamicReloading( ) override { return false; }

	//
	// ICookOnTheFlyModule interface
	//

	virtual TSharedPtr<UE::Cook::ICookOnTheFlyServerConnection> GetDefaultServerConnection() override
	{
		static TSharedPtr<UE::Cook::ICookOnTheFlyServerConnection> DefaultServerConnection = MakeShareable(CreateServerConnection(GetDefaultHostOptions()));
		return DefaultServerConnection;
	}

	virtual TUniquePtr<UE::Cook::ICookOnTheFlyServerConnection> ConnectToServer(const UE::Cook::FCookOnTheFlyHostOptions& HostOptions) override
	{
		return TUniquePtr<UE::Cook::ICookOnTheFlyServerConnection>(CreateServerConnection(HostOptions));
	}

private:
	TUniquePtr<ICookOnTheFlyServerTransport> CreateTransportForHostAddress(const FString& HostIp)
	{
		if (HostIp.StartsWith(TEXT("tcp://")))
		{
			return MakeUnique<FTCPTransport>();
		}

		if (HostIp.StartsWith(TEXT("platform://")) || HostIp.Equals(TEXT("PlatformProtocol"), ESearchCase::IgnoreCase))
		{
			if (FPlatformMisc::GetPlatformHostCommunication().Available())
			{
				return MakeUnique<FPlatformTransport>(EHostProtocol::CookOnTheFly, "CookOnTheFly");
			}
			else
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Platform transport (platform://) not supported for this platform.");

				return nullptr;
			}
		}

		// no transport specified assuming tcp
		return MakeUnique<FTCPTransport>();
	}

	UE::Cook::ICookOnTheFlyServerConnection* CreateServerConnection(const UE::Cook::FCookOnTheFlyHostOptions& HostOptions)
	{
		if (HostOptions.Hosts.IsEmpty())
		{
			return nullptr;
		}

		const double ServerWaitEndTime = FPlatformTime::Seconds() + HostOptions.ServerStartupWaitTime.GetTotalSeconds();

		for (;;)
		{
			for (const FString& Host : HostOptions.Hosts)
			{
				// Try to initialize with each of the IP addresses found in the command line until we 
				// get a working one.

				// find the correct transport for this ip address 
				TUniquePtr<ICookOnTheFlyServerTransport> Transport = CreateTransportForHostAddress(Host);
				if (Transport)
				{
					UE_LOGF(LogCookOnTheFly, Verbose, "Created transport for %ls.", *Host);
					if (Transport->Initialize(*Host))
					{
						UE_LOGF(LogCookOnTheFly, Display, "Created transport for %ls.", *Host);
						return MakeCookOnTheFlyServerConnection(MoveTemp(Transport), Host);
					}
					UE_LOGF(LogCookOnTheFly, Verbose, "Failed to initialize %ls.", *Host);
				}
			}

			if (FPlatformTime::Seconds() > ServerWaitEndTime)
			{
				break;
			}

			FPlatformProcess::Sleep(1.0f);
		};

		UE_LOGF(LogCookOnTheFly, Error, "Failed to connect to COTF server");
		return nullptr;
	}

	UE::Cook::FCookOnTheFlyHostOptions GetDefaultHostOptions()
	{
		TArray<FString> HostIpList;
		FTimespan ServerStartupWaitTime;
		FString HostIpString;
		if (!FParse::Value(FCommandLine::Get(), TEXT("-FileHostIP="), HostIpString))
		{
			if (FParse::Param(FCommandLine::Get(), TEXT("CookOnTheFly")))
			{
				FParse::Value(FCommandLine::Get(), TEXT("-ZenStoreHost="), HostIpString);
			}
		}
		if (HostIpString.ParseIntoArray(HostIpList, TEXT("+"), true) > 0)
		{
			double ServerWaitTimeInSeconds = 0.0;
			FParse::Value(FCommandLine::Get(), TEXT("-CookOnTheFlyServerWaitTime="), ServerWaitTimeInSeconds);
			ServerStartupWaitTime = FTimespan::FromSeconds(ServerWaitTimeInSeconds);
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("ZenPlatformProtocol")) || FParse::Param(FCommandLine::Get(), TEXT("PlatformProtocol")))
		{
			HostIpList.Add("platform://");
		}

		return { HostIpList, ServerStartupWaitTime };
	}
};

IMPLEMENT_MODULE(FCookOnTheFlyModule, CookOnTheFly);
