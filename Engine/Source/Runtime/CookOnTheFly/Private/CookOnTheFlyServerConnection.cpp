// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFlyServerConnection.h"
#include "CookOnTheFly.h"
#include "HAL/PlatformMisc.h"
#include "Async/Async.h"
#include "IPAddress.h"
#include "SocketSubsystem.h"
#include "MultichannelTcpSocket.h"
#include "NetworkMessage.h"
#include "Sockets.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "Serialization/MemoryReader.h"
#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY_STATIC(LogCotfServerConnection, Log, All);

class FCookOnTheFlyServerConnection final
	: public UE::Cook::ICookOnTheFlyServerConnection
{
public:
	FCookOnTheFlyServerConnection(TUniquePtr<ICookOnTheFlyServerTransport> InTransport, const FString& InHost)
		: Transport(MoveTemp(InTransport))
		, Host(InHost)
		, bIsSingleThreaded(!FGenericPlatformProcess::SupportsMultithreading())
	{
		using namespace UE::Cook;

		check(Transport);

		TArray<FString> TargetPlatformNames;
		if (FPlatformProperties::RequiresCookedData())
		{
			GetCookOnTheFlyTargetPlatforms(TargetPlatformNames);
		}
		
		FBufferArchive HandshakeRequestPayload;
		HandshakeRequestPayload << TargetPlatformNames;
		bool bIsSingleThreaded_ = bIsSingleThreaded;
		HandshakeRequestPayload << bIsSingleThreaded_;

		if (!Transport->SendPayload(HandshakeRequestPayload))
		{
			UE_LOGF(LogCotfServerConnection, Fatal, "Failed to send handshake request to server");
			return;
		}

		FArrayReader ResponsePayload;
		if (!Transport->ReceivePayload(ResponsePayload))
		{
			UE_LOGF(LogCotfServerConnection, Fatal, "Failed to receive handshake response from server");
			return;
		}
		bool bIsOk;
		ResponsePayload << bIsOk;
		if (!bIsOk)
		{
			UE_LOGF(LogCotfServerConnection, Fatal, "Couldn't handshake with server");
			return;
		}
		FString ZenHostName;
		ResponsePayload << PlatformName;
		ResponsePayload << ZenProjectName;
		ResponsePayload << ZenHostName;
		ResponsePayload << ZenHostPort;

		// If our server is running its zenserver locally, we get "localhost" as a reply and need to read the remote host name
		if (ZenHostName == "localhost")
		{
			int32 NumRemoteHostNames = 0;
			ResponsePayload << NumRemoteHostNames;
			ZenHostNames.Reserve(NumRemoteHostNames);
			for (int32 RemoteHostIndex = 0; RemoteHostIndex < NumRemoteHostNames; RemoteHostIndex++)
			{
				FString RemoteHostName;
				ResponsePayload << RemoteHostName;
				ZenHostNames.Add(RemoteHostName);
			}
		}
		else
		{
			ZenHostNames.Add(ZenHostName);
		}

		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FCookOnTheFlyServerConnection::OnEnginePreExit);

		Thread = MakeUnique<FThread>(
			TEXT("CotfServerConnection"),
			[this]()
			{
				ThreadEntry();
			},
			[this]()
			{
				SingleThreadedTickFunction();
			},
			8 * 1024);
	}

	~FCookOnTheFlyServerConnection()
	{
		FCoreDelegates::OnEnginePreExit.RemoveAll(this);
		Disconnect();
	}

	virtual const FString& GetHost() const override
	{
		return Host;
	}

	virtual const FString& GetZenProjectName() const override
	{
		return ZenProjectName;
	}

	virtual const TArray<FString> GetZenHostNames() const override
	{
		return ZenHostNames;
	}

	virtual const uint16 GetZenHostPort() const override
	{
		return ZenHostPort;
	}

	virtual const FString& GetPlatformName() const override
	{
		return PlatformName;
	}

	virtual bool IsSingleThreaded() const override
	{
		return bIsSingleThreaded;
	}

	virtual bool IsConnected() const override
	{
		return Transport.IsValid();
	}

	virtual TFuture<UE::Cook::FCookOnTheFlyResponse> SendRequest(UE::Cook::FCookOnTheFlyRequest& Request) override
	{
		using namespace UE::Cook;

		const uint32 CorrelationId					= NextCorrelationId++;
		FCookOnTheFlyMessageHeader& RequestHeader	= Request.GetHeader();

		RequestHeader.MessageStatus = ECookOnTheFlyMessageStatus::Ok;
		RequestHeader.CorrelationId = CorrelationId;
		RequestHeader.Timestamp		= FDateTime::UtcNow().GetTicks();

		FBufferArchive RequestPayload;
		RequestPayload.Reserve(IntCastChecked<int32>(Request.TotalSize()));

		RequestPayload << Request;

		FPendingRequest* PendingRequest = Alloc(CorrelationId);
		PendingRequest->RequestHeader = RequestHeader;
		
		TFuture<FCookOnTheFlyResponse> FutureResponse = PendingRequest->ResponsePromise.GetFuture();

		bool bOk = false;
		if (Transport)
		{
			UE_LOGF(LogCotfServerConnection, Verbose, "Sending: %ls, Size='%lld'", *RequestHeader.ToString(), Request.TotalSize());

			FScopeLock _(&TransportCritical);
			if (Transport->SendPayload(RequestPayload))
			{
				if (bIsSingleThreaded)
				{
					ProcessMessagesWhile([this, CorrelationId]()
						{
							return PendingRequests.Contains(CorrelationId);
						});
				}
				bOk = true;
			}
		}
		if (!bOk)
		{
			UE_LOGF(LogCotfServerConnection, Warning, "Failed to send: %ls, Size='%lld'", *RequestHeader.ToString(), Request.TotalSize());

			FCookOnTheFlyResponse ErrorResponse(Request);
			ErrorResponse.SetStatus(ECookOnTheFlyMessageStatus::Error);
			PendingRequest->ResponsePromise.SetValue(ErrorResponse);

			Free(PendingRequest);
		}
		return FutureResponse;
	}

	DECLARE_DERIVED_EVENT(FCookOnTheFlyServerConnection, UE::Cook::ICookOnTheFlyServerConnection::FMessageEvent, FMessageEvent);
	virtual FMessageEvent& OnMessage() override
	{
		return MessageEvent;
	}

private:
	static void GetCookOnTheFlyTargetPlatforms(TArray<FString>& TargetPlatformNames)
	{
		FPlatformMisc::GetValidTargetPlatforms(TargetPlatformNames);
		// COTF server (a.k.a. UE Editor) appends "Client" to platform names for client-only targets
		// Desktop platforms do the same for runtime client config, but non-desktop platforms don't
		// Append "Client" to platform names, if not already present, in client-only configs to match COTF server
		if (FPlatformProperties::IsClientOnly())
		{
			for (FString& TargetPlatformName : TargetPlatformNames)
			{
				if (!TargetPlatformName.Contains(TEXT("Client")))
				{
					TargetPlatformName.Append(TEXT("Client"));
				}
			}
		}
	}

	void Disconnect()
	{
		if (!bStopRequested.Exchange(true))
		{
			if (Transport)
			{
				Transport->Disconnect();
			}
			Thread->Join();
			Thread.Reset();
		}
	}

	bool ProcessMessagesWhile(TFunctionRef<bool()> Condition)
	{
		using namespace UE::Cook;

		while (Condition())
		{
			FArrayReader MessagePayload;
			if (!Transport || !Transport->ReceivePayload(MessagePayload))
			{
				UE_LOGF(LogCotfServerConnection, Warning, "Failed to receive message from server");
				return false;
			}

			FCookOnTheFlyMessageHeader MessageHeader;
			TArray<uint8> MessageBody;

			MessagePayload << MessageHeader;
			MessagePayload << MessageBody;

			UE_LOGF(LogCotfServerConnection, Verbose, "Received: %ls, Size='%d'", *MessageHeader.ToString(), MessagePayload.Num());

			const bool bIsResponse = EnumHasAnyFlags(MessageHeader.MessageType, ECookOnTheFlyMessage::Response);
			const bool bIsRequest = EnumHasAnyFlags(MessageHeader.MessageType, ECookOnTheFlyMessage::Request);

			if (bIsRequest)
			{
				UE_LOGF(LogCotfServerConnection, Warning, "Received request from server, ignoring");
			}
			else if (bIsResponse)
			{
				FPendingRequest* PendingRequest = GetRequest(MessageHeader.CorrelationId);
				if (!PendingRequest)
				{
					UE_LOGF(LogCotfServerConnection, Warning, "Failed to match response with id '%u' to an existing request", MessageHeader.CorrelationId);
					return false;
				}

				check(PendingRequest->RequestHeader.CorrelationId == MessageHeader.CorrelationId);

				FCookOnTheFlyResponse Response;
				Response.SetHeader(MessageHeader);
				Response.SetBody(MoveTemp(MessageBody));

				PendingRequest->ResponsePromise.SetValue(MoveTemp(Response));

				Free(PendingRequest);
			}
			else
			{
				FCookOnTheFlyMessage Message;
				Message.SetHeader(MessageHeader);
				Message.SetBody(MoveTemp(MessageBody));
				
				if (MessageEvent.IsBound())
				{
					MessageEvent.Broadcast(Message);
				}
			}
		}

		return true;
	}

	void ThreadEntry()
	{
		ProcessMessagesWhile([this]()
			{
				return !bStopRequested.Load();
			});

		UE_LOGF(LogCotfServerConnection, Display, "Terminating connection to server");
		Flush();
		Transport.Reset();
	}

	void SingleThreadedTickFunction()
	{
		ProcessMessagesWhile([this]()
			{
				return Transport && Transport->HasPendingPayload();
			});
	}

	struct FPendingRequest
	{
		UE::Cook::FCookOnTheFlyMessageHeader RequestHeader;
		TPromise<UE::Cook::FCookOnTheFlyResponse> ResponsePromise;
	};

	FPendingRequest* Alloc(uint32 CorrelationId)
	{
		FScopeLock _(&RequestsCriticalSection);
		TUniquePtr<FPendingRequest>& NewPendingRequest = PendingRequests.Add(CorrelationId, MakeUnique<FPendingRequest>());
		return NewPendingRequest.Get();
	}

	void Free(FPendingRequest* PendingRequest)
	{
		FScopeLock _(&RequestsCriticalSection);
		PendingRequests.Remove(PendingRequest->RequestHeader.CorrelationId);
	}

	void Flush()
	{
		if (!PendingRequests.IsEmpty())
		{
			using namespace UE::Cook;
			FCookOnTheFlyRequest SyntheticRequest;
			FScopeLock _(&RequestsCriticalSection);
			for (const TPair<uint32, TUniquePtr<FPendingRequest>>& Pair : PendingRequests)
			{
				FPendingRequest& PendingRequest = *Pair.Value;
				SyntheticRequest.SetHeader(PendingRequest.RequestHeader);
				FCookOnTheFlyResponse ErrorResponse(SyntheticRequest);
				ErrorResponse.SetStatus(ECookOnTheFlyMessageStatus::Error);
				PendingRequest.ResponsePromise.SetValue(ErrorResponse);
			}
			PendingRequests.Empty();
		}
	}
	
	FPendingRequest* GetRequest(uint32 CorrelationId)
	{
		FScopeLock _(&RequestsCriticalSection);
		
		if (TUniquePtr<FPendingRequest>* PendingRequest = PendingRequests.Find(CorrelationId))
		{
			return PendingRequest->Get();
		}

		return nullptr;
	}

	void OnEnginePreExit()
	{
		Disconnect();
	}

	FCriticalSection TransportCritical;
	TUniquePtr<ICookOnTheFlyServerTransport> Transport;
	FString Host;
	FMessageEvent MessageEvent;
	TUniquePtr<FThread> Thread;
	TAtomic<bool> bStopRequested{ false };
	const bool bIsSingleThreaded;
	FString PlatformName;
	FString ZenProjectName;
	TArray<FString> ZenHostNames;
	uint16 ZenHostPort;

	FCriticalSection RequestsCriticalSection;
	TMap<uint32, TUniquePtr<FPendingRequest>> PendingRequests;
	TAtomic<uint32> NextCorrelationId { 1 };
};

UE::Cook::ICookOnTheFlyServerConnection* MakeCookOnTheFlyServerConnection(TUniquePtr<ICookOnTheFlyServerTransport> InTransport, const FString& InHost)
{
	return new FCookOnTheFlyServerConnection(MoveTemp(InTransport), InHost);
}