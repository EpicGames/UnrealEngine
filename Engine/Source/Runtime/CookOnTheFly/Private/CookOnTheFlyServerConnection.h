// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookOnTheFly.h"

class FArrayReader;

class ICookOnTheFlyServerTransport
{
public:
	ICookOnTheFlyServerTransport() {};
	virtual ~ICookOnTheFlyServerTransport() = default;

	virtual bool Initialize(const TCHAR*) = 0;
	virtual void Disconnect() = 0;
	virtual bool SendPayload(const TArray<uint8>& Payload) = 0;
	virtual bool HasPendingPayload() = 0;
	virtual bool ReceivePayload(FArrayReader& Payload) = 0;
};

UE::Cook::ICookOnTheFlyServerConnection* MakeCookOnTheFlyServerConnection(TUniquePtr<ICookOnTheFlyServerTransport> InTransport, const FString& InHost);
