// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformTransport.h"
#include "NetworkMessage.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformHostSocket.h"
#include "GenericPlatform/GenericPlatformHostCommunication.h"
#include "CookOnTheFly.h"

namespace
{
	/**
	 * Socket abstraction required by FNFSMessageHeader that uses IPlatformHostSocket.
	 */
	class FSimpleAbstractSocket_PlatformProtocol : public FSimpleAbstractSocket
	{
	public:

		FSimpleAbstractSocket_PlatformProtocol(IPlatformHostSocketPtr InHostSocket)
			: HostSocket(InHostSocket)
		{
			check(HostSocket != nullptr);
		}

		virtual bool Receive(uint8* Results, int32 Size) const override
		{
			uint64 BytesReceived;
			return (HostSocket->Receive(Results, Size, BytesReceived) == IPlatformHostSocket::EResultNet::Ok);
		}

		virtual bool Send(const uint8* Buffer, int32 Size) const override
		{
			return (HostSocket->Send(Buffer, Size) == IPlatformHostSocket::EResultNet::Ok);
		}

		virtual uint32 GetMagic() const override
		{
			return 0x9E2B83C7;
		}

	private:
		IPlatformHostSocketPtr HostSocket;
	};
}


FPlatformTransport::FPlatformTransport(int32 InProtocolIndex, const FString& InProtocolName)
	: ProtocolIndex(InProtocolIndex)
	, ProtocolName(InProtocolName)
	, HostSocket(nullptr)
{
}


bool FPlatformTransport::Initialize(const TCHAR* InHostIp)
{
	check(HostSocket == nullptr);

	IPlatformHostCommunication& HostCommunication = FPlatformMisc::GetPlatformHostCommunication();

	if (!HostCommunication.Available())
	{
		return false;
	}

	HostSocket = HostCommunication.OpenConnection(ProtocolIndex, ProtocolName);

	if (!HostSocket)
	{
		return false;
	}

	UE_LOGF(LogCookOnTheFly, Display, "Waiting for the server to accept the connection (custom protocol)...");

	// We need to wait because this transport is a bit different than typical sockets.
	// In this case, it's the client (game) that enables the communication by opening the socket.
	// The server (pc) can connect only if it detects this enabled communication protocol.
	return WaitUntilConnected();
}


bool FPlatformTransport::SendPayload(const TArray<uint8>& Payload)
{
	return FNFSMessageHeader::WrapAndSendPayload(Payload, FSimpleAbstractSocket_PlatformProtocol(HostSocket));
}


bool FPlatformTransport::ReceivePayload(FArrayReader& Payload)
{
	return FNFSMessageHeader::ReceivePayload(Payload, FSimpleAbstractSocket_PlatformProtocol(HostSocket));
}


bool FPlatformTransport::WaitUntilConnected()
{
	while (true)
	{
		IPlatformHostSocket::EConnectionState ConnectionState = HostSocket->GetState();

		// We're interested in the Connected state, but break the loop in other non-default states
		// in case any error has appeared.
		if (HostSocket->GetState() != IPlatformHostSocket::EConnectionState::Created)
		{
			break;
		}

		FPlatformProcess::Sleep(0.5f);
	}

	return HostSocket->GetState() == IPlatformHostSocket::EConnectionState::Connected;
}

bool FPlatformTransport::HasPendingPayload()
{
	checkNoEntry();
	return false;
}

void FPlatformTransport::Disconnect()
{
	IPlatformHostCommunication& HostCommunication = FPlatformMisc::GetPlatformHostCommunication();
	if (HostCommunication.Available())
	{
		HostCommunication.CloseConnection(HostSocket);
	}
}

FPlatformTransport::~FPlatformTransport()
{
	if (HostSocket)
	{
		IPlatformHostCommunication& HostCommunication = FPlatformMisc::GetPlatformHostCommunication();

		if (HostCommunication.Available())
		{
			HostCommunication.CloseConnection(HostSocket);

			HostSocket = nullptr;
		}
	}
}
