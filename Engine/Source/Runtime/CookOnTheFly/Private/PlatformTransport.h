// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookOnTheFlyServerConnection.h"
#include "GenericPlatform/GenericPlatformHostSocket.h"


/**
 * Implementation of ITransport using IPlatformHostCommunication/IPlatformHostSocket interfaces (custom target and host pc communication).
 */
class FPlatformTransport
	: public ICookOnTheFlyServerTransport
{
public:

	FPlatformTransport(int32 InProtocolIndex, const FString& InProtocolName);
	~FPlatformTransport();

	//~ Begin ICookOnTheFlyServerTransport interface
	virtual bool Initialize(const TCHAR* HostIp) override;
	virtual bool SendPayload(const TArray<uint8>& Payload) override;
	virtual bool ReceivePayload(FArrayReader& Payload) override;
	virtual bool HasPendingPayload() override;
	virtual void Disconnect() override;
	//~ End ICookOnTheFlyServerTransport interface

private:

	/**
	 * Wait until HostSocket is in a non-default state (preferably Connected).
	 * @return True is the socket is Connected, false otherwise (an error or if the host pc immediately disconnects).
	 */
	bool WaitUntilConnected();

	int32				   ProtocolIndex;
	FString				   ProtocolName;
	IPlatformHostSocketPtr HostSocket;

};
