// Copyright Epic Games, Inc. All Rights Reserved.

#include "TCPTransport.h"
#include "NetworkMessage.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "Sockets.h"
#include "MultichannelTcpSocket.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "CookOnTheFly.h"

FTCPTransport::FTCPTransport()
	:FileSocket(NULL)
	,MCSocket(NULL)
{
}

bool FTCPTransport::Initialize(const TCHAR* InHostIp)
{
	ISocketSubsystem* SSS = ISocketSubsystem::Get();

	FString HostIp = InHostIp;


	// the protocol isn't required for tcp it's assumed by default
	HostIp.RemoveFromStart(TEXT("tcp://"));

	// convert the string to a ip addr structure
	// DEFAULT_TCP_FILE_SERVING_PORT is overridden 
	TSharedRef<FInternetAddr> Addr = SSS->CreateInternetAddr();
	bool bIsValid;

	FIPv4Endpoint FileServerEndpoint;
	if (FIPv4Endpoint::Parse(InHostIp, FileServerEndpoint) || FIPv4Endpoint::FromHostAndPort(InHostIp, FileServerEndpoint))
	{
		Addr->SetIp(FileServerEndpoint.Address.Value);
		Addr->SetPort(FileServerEndpoint.Port == 0 ? DEFAULT_TCP_FILE_SERVING_PORT : FileServerEndpoint.Port);
		bIsValid = true;
	}
	else
	{
		Addr->SetIp(*HostIp, bIsValid);
		Addr->SetPort(DEFAULT_TCP_FILE_SERVING_PORT);
	}

	if (bIsValid)
	{
		// create the socket
		FileSocket = SSS->CreateSocket(NAME_Stream, TEXT("COTF tcp"), Addr->GetProtocolType());

		// try to connect to the server
		if (FileSocket == nullptr || FileSocket->Connect(*Addr) == false)
		{
			// on failure, shut it all down
			SSS->DestroySocket(FileSocket);
			FileSocket = NULL;
			UE_LOGF(LogCookOnTheFly, Error, "Failed to connect to COTF server at %ls.", *Addr->ToString(true));
		}
		HostName = Addr->ToString(false);
	}

#if USE_MCSOCKET_FOR_NFS
	MCSocket = new FMultichannelTcpSocket(FileSocket, 64 * 1024 * 1024);
#endif

	return ( FileSocket || MCSocket );
}


bool FTCPTransport::SendPayload(const TArray<uint8>& Payload)
{
#if USE_MCSOCKET_FOR_NFS
	return FNFSMessageHeader::WrapAndSendPayload(Payload, FSimpleAbstractSocket_FMultichannelTCPSocket(MCSocket, NFS_Channels::Main));
#else 	
	return FNFSMessageHeader::WrapAndSendPayload(Payload, FSimpleAbstractSocket_FSocket(FileSocket));
#endif 
}


bool FTCPTransport::ReceivePayload(FArrayReader& Payload)
{
#if USE_MCSOCKET_FOR_NFS
	return FNFSMessageHeader::ReceivePayload(Payload, FSimpleAbstractSocket_FMultichannelTCPSocket(MCSocket, NFS_Channels::Main));
#else
	return FNFSMessageHeader::ReceivePayload(Payload, FSimpleAbstractSocket_FSocket(FileSocket));
#endif
}

bool FTCPTransport::HasPendingPayload()
{
#if USE_MCSOCKET_FOR_NFS
	return MCSocket->DataAvailable(NFS_Channels::Main) > 0;
#else
	uint32 PendingDataSize;
	return FileSocket->HasPendingData(PendingDataSize);
#endif
}

void FTCPTransport::Disconnect()
{
#if USE_MCSOCKET_FOR_NFS
	checkNoEntry();
#else
	FileSocket->Close();
#endif
}

FTCPTransport::~FTCPTransport()
{
	// on failure, shut it all down
	delete MCSocket;
	MCSocket = NULL;
	ISocketSubsystem::Get()->DestroySocket(FileSocket);
	FileSocket = NULL;
}
