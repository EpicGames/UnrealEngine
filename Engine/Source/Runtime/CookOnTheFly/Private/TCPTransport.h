// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookOnTheFlyServerConnection.h"

class FTCPTransport
	: public ICookOnTheFlyServerTransport
{

public:

	FTCPTransport();
	~FTCPTransport();

	// ITransport Interface. 
	virtual bool Initialize(const TCHAR* HostIp) override;
	virtual bool SendPayload(const TArray<uint8>& Payload) override;
	virtual bool ReceivePayload(FArrayReader& Payload) override;
	virtual bool HasPendingPayload() override;
	virtual void Disconnect() override;

private: 

	class FSocket*		FileSocket;
	class FMultichannelTcpSocket* MCSocket;
	FString HostName;
};
